{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Direct Mesh Control",
	"Description": "Animate using click & drag and surface selection.",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com/",
	"DocsURL": "",
	"MarketplaceURL": "",
	"EnabledByDefault" : false,
	"CanContainContent": true,
	"IsExperimentalVersion": true,
	"IsBetaVersion": false,
	"Installed": false,
	"Plugins": [
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "SkeletalMeshModelingTools",
			"Enabled": true,
			"TargetAllowList": [ "Editor" ]
		},
		{
			"Name": "ControlRig",
			"Enabled": true
		},
		{
			"Name": "RigVM",
			"Enabled": true
		},
		{
			"Name": "ComputeFramework",
			"Enabled": true
		},
		{
			"Name": "DeformerGraph",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "DirectMeshControl",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "DirectMeshControlRig",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	]
}