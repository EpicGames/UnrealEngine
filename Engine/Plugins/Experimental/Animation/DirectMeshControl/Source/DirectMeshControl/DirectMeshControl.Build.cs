// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DirectMeshControl : ModuleRules
{
	public DirectMeshControl(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicDependencyModuleNames.AddRange(new string[]
		{
			"ModelingComponents"
		});
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ComputeFramework",
				"Core",
				"CoreUObject",
				"DeveloperSettings",
				"DynamicMesh",
				"EditorInteractiveToolsFramework",
				"EditorSubsystem",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"InputCore",
				"InteractiveToolsFramework",
				"MeshConversion",
				"MeshDescription",
				"MeshModelingToolsEditorOnly",
				"ModelingToolsEditorMode",
				"OptimusCore",
				"Persona",
				"RenderCore",
				"SkeletalMeshDescription",
				"SkeletalMeshModelingTools",
				"SkeletalMeshUtilitiesCommon",
				"Slate",
				"SlateCore",
				"StaticMeshDescription",
				"UnrealEd"
			}
		);
	}
}
