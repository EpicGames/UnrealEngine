// Copyright Epic Games, Inc. All Rights Reserved.

#include "DMCMeshGenerationSubsystem.h"

#include "Editor.h"
#include "Misc/SecureHash.h"
#include "Engine/SkeletalMesh.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DMCMeshGenerationSubsystem)

namespace UE::DMC
{

uint32 GetDDCHash(USkeletalMesh* InSkeletalMesh)
{
	FSHAHash SHAHash;
	if (InSkeletalMesh)
	{
		const FString DDCKey = InSkeletalMesh->GetDerivedDataKey();
		FSHA1 SHA1;
		SHA1.UpdateWithString(*DDCKey, DDCKey.Len());
		SHA1.Final();
		SHA1.GetHash(SHAHash.Hash);
	}
	return HashCombine(GetTypeHash(InSkeletalMesh), GetTypeHash(SHAHash));
}
	
}

using namespace UE::DMC;
using namespace UE::Geometry;

void UDMCMeshGenerationManager::Shutdown()
{
	SubSkeletalMeshes.Reset();
}

const FGroupSubMeshes& UDMCMeshGenerationManager::GetSubMeshes(USkeletalMesh* InSkeletalMesh, const FDynamicMesh3* Mesh, const FName LayerName)
{
	if (!InSkeletalMesh || !Mesh)
	{
		static FGroupSubMeshes Dummy;
		return Dummy;
	}
	
	FSubSkeletalMeshData& Data = SubSkeletalMeshes.FindOrAdd({InSkeletalMesh, LayerName});

	const uint32 Hash = GetDDCHash(InSkeletalMesh);
	if (Data.Hash != Hash)
	{
		Data.Hash = Hash;
		Data.GroupSubMeshes.Rebuild(InSkeletalMesh, Mesh, LayerName);
	}
	else
	{
		// check consistency
	}

	return Data.GroupSubMeshes;
}

bool UDMCMeshGenerationSubsystem::bIsShuttingDown = false;

UDMCMeshGenerationSubsystem* UDMCMeshGenerationSubsystem::Get()
{
	if (GEditor && !bIsShuttingDown)
	{
		return GEditor->GetEditorSubsystem<UDMCMeshGenerationSubsystem>();
	}
	return nullptr;
}

void UDMCMeshGenerationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	bIsShuttingDown = false;
	GenerationManager = NewObject<UDMCMeshGenerationManager>(this);

	if (GEditor)
	{
		GEditor->OnEditorClose().AddUObject(this, &UDMCMeshGenerationSubsystem::OnShutdown);
	}
	FCoreDelegates::OnEnginePreExit.AddUObject(this, &UDMCMeshGenerationSubsystem::OnShutdown);
}

void UDMCMeshGenerationSubsystem::Deinitialize()
{
	if (GEditor)
	{
		GEditor->OnEditorClose().RemoveAll(this);
	}
	FCoreDelegates::OnEnginePreExit.RemoveAll(this);
	bIsShuttingDown = true;

	if (GenerationManager)
	{
		GenerationManager->Shutdown();
		GenerationManager = nullptr;
	}
}

void UDMCMeshGenerationSubsystem::OnShutdown()
{
	bIsShuttingDown = true;
}
