// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "DirectMeshControlUtilities.h"

#include "DMCMeshGenerationSubsystem.generated.h"

#define UE_API DIRECTMESHCONTROL_API

class USkeletalMesh;
class UMaterialInstanceDynamic;
class UOptimusDeformer;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

/**
 * UDMCMeshGenerationManager is a persistent cache manager for (skeletal mesh, layer name) to sub-meshes mappings.
 */

UCLASS(MinimalAPI)
class UDMCMeshGenerationManager : public UObject
{
	GENERATED_BODY()

public:
	/** Clears all cached sub-mesh entries, releasing references to generated assets. */
	UE_API virtual void Shutdown();

	/**
	 * Returns the cached FGroupSubMeshes for InSkeletalMesh / LayerName, rebuilding the entry when the DDC hash indicates the source data has changed.
	 * @param InSkeletalMesh  Source skeletal mesh.
	 * @param Mesh            Current dynamic mesh representation used to detect changes and rebuild.
	 * @param LayerName       Triangle label layer that encodes polygroup IDs.
	 * @return Reference to the up-to-date cached FGroupSubMeshes.
	 */
	UE_API const FGroupSubMeshes& GetSubMeshes(USkeletalMesh* InSkeletalMesh, const UE::Geometry::FDynamicMesh3* Mesh, const FName LayerName);
	
protected:
	/** The set of cached entries keyed by (SkeletalMesh, LayerName). */
	UPROPERTY()
	TSet<FSubSkeletalMeshData> SubSkeletalMeshes;
};
	
/**
 * UDMCMeshGenerationSubsystem is an editor subsystem that provides global access to UDMCMeshGenerationManager.
 */

UCLASS(MinimalAPI)
class UDMCMeshGenerationSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:
	/** Returns the active UDMCMeshGenerationSubsystem instance, or nullptr if the engine is shutting down. */
	static UDMCMeshGenerationSubsystem* Get();

	/**
	 * Creates the UDMCMeshGenerationManager and binds the engine-exit and pre-exit shutdown callbacks.
	 * @param Collection  Subsystem collection (passed to the parent implementation).
	 */
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;

	/** Removes bound shutdown callbacks, calls Shutdown() on the manager, and nulls the @c GenerationManager pointer. */
	UE_API virtual void Deinitialize() override;

	/** The owned mesh generation manager. Accessed via Get()->GenerationManager. */
	UPROPERTY()
	TObjectPtr<UDMCMeshGenerationManager> GenerationManager = nullptr;

protected:
	/** Sets bIsShuttingDown to true so that subsequent Get() calls return nullptr. */
	UE_API virtual void OnShutdown();

private:
	/** Static flag set to true during engine shutdown to guard Get() against use-after-destroy. */
	static UE_API bool bIsShuttingDown;
};
	
#undef UE_API
