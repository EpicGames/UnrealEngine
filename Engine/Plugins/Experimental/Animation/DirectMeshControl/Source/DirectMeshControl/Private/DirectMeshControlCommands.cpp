// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectMeshControlCommands.h"
#include "InputCoreTypes.h"
#include "DirectMeshControlStyle.h"

#define LOCTEXT_NAMESPACE "DirectMeshControlCommands"

FDirectMeshControlCommands::FDirectMeshControlCommands() :
	TCommands<FDirectMeshControlCommands>(
		"DirectMeshControlCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "DirectMeshControlCommands", "Direct Mesh Control Extension"), // Localized context name for displaying
		NAME_None, // Parent
		FDirectMeshControlStyle::Get()->GetStyleSetName() // Icon Style Set
		)
{
}

void FDirectMeshControlCommands::RegisterCommands()
{
	UI_COMMAND(BeginDirectMeshControlTools, "DMC", "Direct Mesh Control", EUserInterfaceActionType::ToggleButton, FInputChord());
	
	UI_COMMAND(BeginDirectMeshPolygroupTool, "Generate DMC", "Generate polygroups for surface selection", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE