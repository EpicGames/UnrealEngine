// Copyright Epic Games, Inc. All Rights Reserved.


#include "DirectMeshControlComponent.h"

#include "SkeletalMeshDeformerHelpers.h"
#include "SkeletalRenderPublic.h"
#include "Animation/MeshDeformerInstance.h"
#include "Engine/SkeletalMesh.h"
#include "Rendering/SkeletalMeshRenderData.h"

void UDirectMeshControlComponent::SendRenderDynamicData_Concurrent()
{
#if WITH_STATE_STREAM_ACTOR
	return;
#endif

	USkeletalMesh* SkeletalMeshPtr = GetSkeletalMeshAsset();
	
#if WITH_EDITOR
	if (SkeletalMeshPtr && SkeletalMeshPtr->IsCompiling())
	{
		return;
	}
#endif
	
	// skinned mesh component override
	// USkinnedMeshComponent::SendRenderDynamicData_Concurrent();

	if (MeshObject && SkeletalMeshPtr && (bRecentlyRendered || VisibilityBasedAnimTickOption == EVisibilityBasedAnimTickOption::AlwaysTickPoseAndRefreshBones || GIsEditor || !MeshObject->bHasBeenUpdatedAtLeastOnce))
	{
		const int32 MinLodIndex = ComputeMinLOD();
		const int32 MaxLODIndex = MeshObject->GetSkeletalMeshRenderData().LODRenderData.Num() - 1;
		int32 UseLOD = FMath::Clamp(GetPredictedLODLevel(), MinLodIndex, MaxLODIndex);
		
		// Only update the state if PredictedLODLevel is valid
		FSkeletalMeshRenderData* SkelMeshRenderData = GetSkeletalMeshRenderData();
		if (SkelMeshRenderData && SkelMeshRenderData->LODRenderData.IsValidIndex(UseLOD))
		{
			ActiveMorphTargets.Empty();
			MeshObject->Update(
				UseLOD,
				this,
				ActiveMorphTargets,
				MorphTargetWeights,
				EPreviousBoneTransformUpdateMode::None,
				GetExternalMorphWeights(UseLOD)
			);  // send to rendering thread

			MeshObject->bHasBeenUpdatedAtLeastOnce = true;
		}

		if (UMeshDeformerInstance* DeformerInstanceForLOD = GetMeshDeformerInstanceForLOD(UseLOD))
		{
#if WITH_EDITORONLY_DATA
			for (TUniquePtr<FMeshDeformerGeometryReadbackRequest>& Request : GeometryReadbackRequests)
			{
				DeformerInstanceForLOD->RequestReadbackDeformerGeometry(MoveTemp(Request));
			}
#endif // WITH_EDITORONLY_DATA
			
			UMeshDeformerInstance::FEnqueueWorkDesc Desc;
			Desc.Scene = GetScene();
			Desc.OwnerName = SkeletalMeshPtr ? SkeletalMeshPtr->GetFName() : GetFName();
			Desc.ExecutionGroup = UMeshDeformerInstance::ExecutionGroup_BeginInitViews;
		
			// Fallback is to reset the passthrough vertex factory if the deformer fails to run.
			Desc.FallbackDelegate.BindLambda([MeshObjectPtr = MeshObject, UseLOD]()
			{
				FSkeletalMeshDeformerHelpers::ResetVertexFactoryBufferOverrides(MeshObjectPtr, UseLOD);
			});
			DeformerInstanceForLOD->EnqueueWork(Desc);
		}
	}

	UActorComponent::SendRenderDynamicData_Concurrent();

#if WITH_EDITORONLY_DATA
	// Avoid unbound accumulation of readback requests, each readback request must be picked up by the deformer instance within the same frame or ignored otherwise
	GeometryReadbackRequests.Reset();
#endif // WITH_EDITORONLY_DATA 
}