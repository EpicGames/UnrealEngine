// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectMeshControlModule.h"

#include "DirectMeshControlCommands.h"
#include "DirectMeshControlStyle.h"
#include "ModelingModeToolExtensions.h"
#include "Features/IModularFeatures.h"
#include "Tools/DirectMeshPolygroupTool.h"

#include "InteractiveToolManager.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "FDirectMeshControlModule"

void FDirectMeshControlModule::StartupModule()
{
	FDirectMeshControlStyle::Initialize();
	FDirectMeshControlCommands::Register();

	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
}

void FDirectMeshControlModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);

	FDirectMeshControlCommands::Unregister();
	FDirectMeshControlStyle::Shutdown();
}

FText FDirectMeshControlModule::GetExtensionName()
{
	return LOCTEXT("ExtensionName", "DMCTools");
}

FText FDirectMeshControlModule::GetToolSectionName()
{
	return LOCTEXT("SectionName", "DMC");
}

bool FDirectMeshControlModule::GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut)
{
	InfoOut.ExtensionCommand = FDirectMeshControlCommands::Get().BeginDirectMeshControlTools;
	
	return true;
}

void FDirectMeshControlModule::GetExtensionTools(const FExtensionToolQueryInfo& InQueryInfo, TArray<FExtensionToolDescription>& OutTools)
{
	{
		FExtensionToolDescription DirectMeshPolygroupToolInfo;
		DirectMeshPolygroupToolInfo.ToolName = LOCTEXT("DirectMeshPolygroupTool", "Direct Mesh Polygroup");
		DirectMeshPolygroupToolInfo.ToolCommand = FDirectMeshControlCommands::Get().BeginDirectMeshPolygroupTool;
		DirectMeshPolygroupToolInfo.ToolBuilder = NewObject<UDirectMeshPolygroupToolBuilder>();
		OutTools.Add(DirectMeshPolygroupToolInfo);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDirectMeshControlModule, DirectMeshControl)