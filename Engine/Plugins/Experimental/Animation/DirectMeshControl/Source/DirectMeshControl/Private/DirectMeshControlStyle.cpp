// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectMeshControlStyle.h"

#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "SlateOptMacros.h"

TSharedPtr< FSlateStyleSet > FDirectMeshControlStyle::StyleSet = nullptr;

#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr< class ISlateStyle > FDirectMeshControlStyle::Get()
{
	return StyleSet;
}

FName FDirectMeshControlStyle::GetStyleSetName()
{
	static FName StyleName(TEXT("DirectMeshControlStyle"));
	return StyleName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FDirectMeshControlStyle::Initialize()
{
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}
	
	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	static const FVector2D IconSize20x20(20.0f, 20.0f);
	
	{
		StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/Animation/DirectMeshControl/Resources"));
		StyleSet->Set("DirectMeshControlCommands.BeginDirectMeshControlTools", new IMAGE_BRUSH_SVG("DirectMeshControl_20", IconSize20x20));
	}
	
	{
		StyleSet->SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/ModelingToolsEditorMode/Content"));
		StyleSet->Set("DirectMeshControlCommands.BeginDirectMeshPolygroupTool", new IMAGE_BRUSH("Icons/PolyGroups_40x", IconSize20x20));
	}
	
	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FDirectMeshControlStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}