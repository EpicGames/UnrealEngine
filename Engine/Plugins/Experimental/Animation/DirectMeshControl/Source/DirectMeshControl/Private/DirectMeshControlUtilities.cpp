// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectMeshControlUtilities.h"

#include "Animation/Skeleton.h"
#include "DMCMeshGenerationSubsystem.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicSubmesh3.h"
#include "Engine/SkinnedAssetCommon.h"
#include "Logging/LogScopedVerbosityOverride.h"
#include "Materials/Material.h"
#include "MeshConversionOptions.h"
#include "MeshDescription.h"
#include "Misc/ScopedSlowTask.h"
#include "OptimusDeformer.h"
#include "Polygroups/GroupSetAdapter.h"
#include "Polygroups/PolygroupUtil.h"
#include "SkeletalMeshAttributes.h"
#include "SkinnedAssetCompiler.h"
#include "Rendering/SkeletalMeshModel.h"
#include "StaticToSkeletalMeshConverter.h"
#include "UObject/Package.h"
#include "Util/ColorConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DirectMeshControlUtilities)

namespace UE::DMC
{
	
const FName& GetSubToSourceAttrName()
{
	static FName VtxMappingAttrName(TEXT("SubToSource"));
	return VtxMappingAttrName;
}

const FName& GetColorVarName()
{
	static FName VtxColorAttrName(TEXT("OverlayColor"));
	return VtxColorAttrName;
}

UMaterial* GetMaterial()
{
	static TWeakObjectPtr<UMaterial> Material = nullptr;
	if (!Material.IsValid())
	{
		static FString MaterialPath(TEXT("/DirectMeshControl/Materials/M_DirectMeshControl"));
		Material = LoadObject<UMaterial>(nullptr, MaterialPath);
		ensure(Material.IsValid());
	}
	return Material.Get();
}

USkeleton* GetSkeleton()
{
	static TWeakObjectPtr<USkeleton> WeakSkeleton = nullptr;
	if (!WeakSkeleton.IsValid())
	{
		UDMCMeshGenerationSubsystem* Subsystem = UDMCMeshGenerationSubsystem::Get();
		if (ensure(Subsystem))
		{
			WeakSkeleton = NewObject<USkeleton>(Subsystem);
			{
				const TCHAR* RootBoneName = TEXT("Root"); 
				FTransform RootTransform(FTransform::Identity);
				FReferenceSkeletonModifier Modifier(WeakSkeleton.Get());
				Modifier.Add(FMeshBoneInfo(RootBoneName, RootBoneName, INDEX_NONE), RootTransform);
			}
		}
	}
	return WeakSkeleton.Get();
}

UOptimusDeformer* GetDeformer()
{
	static TWeakObjectPtr<UOptimusDeformer> Deformer = nullptr;
	if (!Deformer.IsValid())
	{
		static FString DeformerPath(TEXT("/DirectMeshControl/Deformers/DG_DirectMeshControl"));
		Deformer = LoadObject<UOptimusDeformer>(nullptr, DeformerPath);
		ensure(Deformer.IsValid());
	}
	return Deformer.Get();
}
	
const FGroupSubMeshes& GetSubMeshes(USkeletalMesh* InSkeletalMesh, const Geometry::FDynamicMesh3* Mesh, const FName LayerName)
{
	static const FGroupSubMeshes Dummy;
	if (!InSkeletalMesh || !Mesh)
	{
		return Dummy;
	}

	UDMCMeshGenerationSubsystem* Subsystem = UDMCMeshGenerationSubsystem::Get();
	if (!ensure(Subsystem))
	{
		return Dummy;
	}

	return Subsystem->GenerationManager ? Subsystem->GenerationManager->GetSubMeshes(InSkeletalMesh, Mesh, LayerName) : Dummy;
}
	
}

using namespace UE::DMC;
using namespace UE::Geometry;

void FGroupSubMeshes::Rebuild(USkeletalMesh* InSkeletalMesh, const FDynamicMesh3* Mesh, const FName LayerName)
{
	Reset();

	UDMCMeshGenerationSubsystem* Subsystem = UDMCMeshGenerationSubsystem::Get();
	if (!ensure(Subsystem))
	{
		return;
	}

	const FDynamicMeshTriangleLabelAttribute* SelectionLayer = FindTriangleLabelLayerByName(*Mesh, LayerName);
	if (!ensure(SelectionLayer))
	{
		return;
	}

	const FTriangleReadOnlyLabelAdapter Adapter(Mesh, SelectionLayer);
	
	TMap<int32, TArray<int32>> PolyGroupToTri;
	for (int32 TriIndex = 0; TriIndex < Mesh->TriangleCount(); TriIndex++)
	{
		const int32 GroupID = Adapter.GetGroup(TriIndex);
		PolyGroupToTri.FindOrAdd(GroupID).Add(TriIndex);
	}

	if (PolyGroupToTri.IsEmpty())
	{
		return;
	}
	
	FSkeletalMeshModel* SkeletalMeshModel = InSkeletalMesh->GetImportedModel();
	if (!SkeletalMeshModel || SkeletalMeshModel->LODModels.IsEmpty())
	{
		return;
	}

	USkeleton* Skeleton = GetSkeleton();
	UMaterial* Material = GetMaterial();
	if (!Skeleton || !Material)
	{
		return;
	}
	
	const TArray<uint32>& RenderToImportedMap = SkeletalMeshModel->LODModels[0].GetRawPointIndices();
	const TArray<FSkeletalMaterial> Materials({Material});
	const TArray<FName> AttributesToEnableForRender({GetSubToSourceAttrName()});
	
	FStaticToSkeletalMeshConverter::FInitializationParams Parameters;
	Parameters.Materials = Materials;
	Parameters.bRecomputeNormals = false;
	Parameters.bRecomputeTangents = false;
	Parameters.bCacheOptimize = true;
	Parameters.VertexAttributesForRender = AttributesToEnableForRender;
	
	// no need for useless logs
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSkeletalMesh, ELogVerbosity::Warning);
	
	for (const auto& [GroudID, Triangles]: PolyGroupToTri)
	{
		FMeshDescription MeshDescription;
		{
			FDynamicSubmesh3 SubMesh(Mesh, Triangles);
			if (FDynamicMeshAttributeSet* SubAttributes = SubMesh.GetSubmesh().Attributes())
			{
				SubAttributes->DisableMaterialID();
				SubAttributes->DisableBones();
			}
					
			// Full binding to the root bone.
			FSkeletalMeshAttributes Attributes(MeshDescription);
			Attributes.Register();
			FConversionToMeshDescriptionOptions ConverterOptions;
			ConverterOptions.bSetPolyGroups = false;
			ConverterOptions.MaxValidMaterialID = 0;
			FDynamicMeshToMeshDescription Converter(ConverterOptions);
			Converter.Convert(&SubMesh.GetSubmesh(), MeshDescription);

			// Full binding to the root bone.
			Attributes.Bones().Reset();
			const FBoneID BoneID = Attributes.CreateBone();
			Attributes.GetBoneNames().Set(BoneID, TEXT("Root"));
			Attributes.GetBoneParentIndices().Set(BoneID, INDEX_NONE);
			Attributes.GetBonePoses().Set(BoneID, FTransform::Identity);
			
			constexpr int32 RootBoneIndex = 0;
			FSkinWeightsVertexAttributesRef SkinWeights = Attributes.GetVertexSkinWeights();
			UE::AnimationCore::FBoneWeight RootInfluence(RootBoneIndex, 1.0f);
			UE::AnimationCore::FBoneWeights RootBinding = UE::AnimationCore::FBoneWeights::Create({ RootInfluence });
			for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
			{
				SkinWeights.Set(VertexID, RootBinding);
			}

			// vertex mapping attributes
			MeshDescription.VertexAttributes().RegisterAttribute<float>(GetSubToSourceAttrName(), 1, 0.5f, EMeshAttributeFlags::None);
			TVertexAttributesRef<float> VtxMappingAttr = MeshDescription.VertexAttributes().GetAttributesRef<float>(GetSubToSourceAttrName());

			for (const FVertexID VertexID : MeshDescription.Vertices().GetElementIDs())
			{
				const int32 BaseIndex = SubMesh.MapVertexToBaseMesh(VertexID);
				const int32 RenderIndex = RenderToImportedMap.Find(BaseIndex);
				if (ensure(RenderToImportedMap.IsValidIndex(RenderIndex)))
				{
					VtxMappingAttr.Set(VertexID, static_cast<float>(RenderIndex) + 0.5f);
				}
			}

			// vertex colors
			const FVector4f Color(LinearColors::SelectFColor(GroudID));
			TVertexInstanceAttributesRef<FVector4f> VertexColor = Attributes.GetVertexInstanceColors();
			for (int32 ColorIndex = 0; ColorIndex < VertexColor.GetNumElements(); ColorIndex++)
			{
				VertexColor.Set(ColorIndex, Color);
			}
		}

		const FString Name = FString::Printf(TEXT("%s_%s_%d"), *InSkeletalMesh->GetName(), *LayerName.ToString(), GroudID);
		const bool bFound = StaticFindObjectFastInternal(USkeletalMesh::StaticClass(), Subsystem->GenerationManager, *Name) != nullptr;
		const FName UniqueName = bFound ? MakeUniqueObjectName(Subsystem->GenerationManager, USkeletalMesh::StaticClass(), *Name) : *Name;
		
		USkeletalMesh* SubSkeletalMesh = NewObject<USkeletalMesh>(Subsystem->GenerationManager, UniqueName, RF_NoFlags);
		{			
			// increment the PostEditChangeStackCounter to batch-build the render data ourselves
			constexpr bool bNoPostEditChange = false, bNoReregister = false;
			FScopedSkeletalMeshPostEditChange Dummy(SubSkeletalMesh, bNoPostEditChange, bNoReregister);
			
			TArray<const FMeshDescription*> MeshDescriptions({&MeshDescription});
			FStaticToSkeletalMeshConverter::InitializeSkeletalMeshFromMeshDescriptions(
				SubSkeletalMesh,
				MeshDescriptions,
				Skeleton->GetReferenceSkeleton(),
				Parameters);
		}
		SubSkeletalMesh->SetSkeleton(Skeleton);
		
		const int32 Index = SubSkeletalMeshes.Add(SubSkeletalMesh);
		GroupIdToSubMesh.Emplace(GroudID, Index);
	}
	
	// batch-build the render data
	if (!SubSkeletalMeshes.IsEmpty())
	{
		TArray<USkinnedAsset*> MeshesToFinalize;
		MeshesToFinalize.Reserve(SubSkeletalMeshes.Num());
		for (const TObjectPtr<USkeletalMesh>& SkeletalMeshPtr: SubSkeletalMeshes)
		{
			USkeletalMesh* SkeletalMesh = SkeletalMeshPtr.Get();
			if (ensure(SkeletalMesh))
			{
				SkeletalMesh->Build();
				MeshesToFinalize.Add(SkeletalMesh);
			}
		}
		
		if (!MeshesToFinalize.IsEmpty())
		{
			FScopedSlowTask SlowTask(0.f, NSLOCTEXT("DirectMeshControl", "DirectMeshControl_Rebuild", "Finalizing Direct Mesh Controls..."));
			SlowTask.MakeDialog();
			FSkinnedAssetCompilingManager::Get().FinishCompilation(MeshesToFinalize);
		}
	}
}

void FGroupSubMeshes::Reset()
{
	for (TObjectPtr<USkeletalMesh>& WeakSubMesh: SubSkeletalMeshes)
	{
		if (WeakSubMesh.Get())
		{
			WeakSubMesh->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			WeakSubMesh->MarkAsGarbage();
			WeakSubMesh = nullptr;
		}
	}
	SubSkeletalMeshes.Reset();
	GroupIdToSubMesh.Reset();
}

const TArray<TObjectPtr<USkeletalMesh>>& FGroupSubMeshes::GetSubSkeletalMeshes() const
{
	return SubSkeletalMeshes;
}
	
const TMap<int32, int32>& FGroupSubMeshes::GetGroupIdToSubMesh() const
{
	return GroupIdToSubMesh;
}
