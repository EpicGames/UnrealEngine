// Copyright Epic Games, Inc. All Rights Reserved.

#include "OptimusDirectMeshControlComponentSource.h"

#include "DirectMeshControlComponent.h"
#include "Rendering/SkeletalMeshLODRenderData.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "SkeletalRenderPublic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OptimusDirectMeshControlComponentSource)


#define LOCTEXT_NAMESPACE "OptimusDirectMeshControlComponentSource"


FName UOptimusDirectMeshControlComponentSource::Domains::Vertex("Vertex");
FName UOptimusDirectMeshControlComponentSource::Domains::Triangle("Triangle");
FName UOptimusDirectMeshControlComponentSource::Domains::DuplicateVertex("DuplicateVertex");


FText UOptimusDirectMeshControlComponentSource::GetDisplayName() const
{
	return LOCTEXT("DirectMeshControlComponent", "Direct Mesh Control Component");
}

FName UOptimusDirectMeshControlComponentSource::GetBindingName() const
{
	return FName("DirectMeshControl");
}

TSubclassOf<UActorComponent> UOptimusDirectMeshControlComponentSource::GetComponentClass() const
{
	return UDirectMeshControlComponent::StaticClass();
}


TArray<FName> UOptimusDirectMeshControlComponentSource::GetExecutionDomains() const
{
	return {Domains::Vertex, Domains::Triangle};
}

int32 UOptimusDirectMeshControlComponentSource::GetLodIndex(const UActorComponent* InComponent) const
{
	const UDirectMeshControlComponent* DMCComponent = Cast<UDirectMeshControlComponent>(InComponent);
	return DMCComponent ? DMCComponent->GetPredictedLODLevel() : 0;
}

uint32 UOptimusDirectMeshControlComponentSource::GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const
{
	const UDirectMeshControlComponent* DMCComponent = Cast<UDirectMeshControlComponent>(InComponent);
	if (!DMCComponent)
	{
		return 0;
	}

	const FSkeletalMeshObject* SkeletalMeshObject = DMCComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return 0;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.LODRenderData.IsValidIndex(InLod) ? 
		&SkeletalMeshRenderData.LODRenderData[InLod] : nullptr;

	return LodRenderData ? LodRenderData->RenderSections.Num() : 0;
}

bool UOptimusDirectMeshControlComponentSource::GetComponentElementCountsForExecutionDomain(
	FName InDomainName,
	const UActorComponent* InComponent,
	int32 InLodIndex,
	TArray<int32>& OutInvocationElementCounts
	) const
{
	const UDirectMeshControlComponent* DMCComponent = Cast<UDirectMeshControlComponent>(InComponent);
	if (!DMCComponent)
	{
		return false;
	}

	const FSkeletalMeshObject* SkeletalMeshObject = DMCComponent->MeshObject;
	if (!SkeletalMeshObject)
	{
		return false;
	}

	FSkeletalMeshRenderData const& SkeletalMeshRenderData = SkeletalMeshObject->GetSkeletalMeshRenderData();
	FSkeletalMeshLODRenderData const* LodRenderData = SkeletalMeshRenderData.LODRenderData.IsValidIndex(InLodIndex) ? 
		&SkeletalMeshRenderData.LODRenderData[InLodIndex] : nullptr;
	
	if (!LodRenderData)
	{
		return false;
	}

	OutInvocationElementCounts.Reset();
	
	if (InDomainName == Domains::Triangle || InDomainName == Domains::Vertex)
	{
		const int32 NumInvocations = LodRenderData->RenderSections.Num();

		OutInvocationElementCounts.Reset();
		OutInvocationElementCounts.Reserve(NumInvocations);
		for (int32 InvocationIndex = 0; InvocationIndex < NumInvocations; ++InvocationIndex)
		{
			FSkelMeshRenderSection const& RenderSection = LodRenderData->RenderSections[InvocationIndex];
			const int32 NumThreads = InDomainName == Domains::Vertex ? RenderSection.NumVertices : RenderSection.NumTriangles;
			OutInvocationElementCounts.Add(NumThreads);
		}
	
		return true;
	}

	// Unknown execution domain.
	return false;
}

#undef LOCTEXT_NAMESPACE
