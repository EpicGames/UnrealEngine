// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "OptimusComponentSource.h"

#include "OptimusDirectMeshControlComponentSource.generated.h"


UCLASS()
class UOptimusDirectMeshControlComponentSource : public UOptimusComponentSource
{
	GENERATED_BODY()
public:
	struct Domains
	{
		static FName Vertex;
		static FName Triangle;
		static FName DuplicateVertex;
	};
	
	// UOptimusComponentSource implementations
	virtual FText GetDisplayName() const override;
	virtual FName GetBindingName() const override;
	virtual TSubclassOf<UActorComponent> GetComponentClass() const override;
	virtual TArray<FName> GetExecutionDomains() const override;
	virtual int32 GetLodIndex(const UActorComponent* InComponent) const override;
	virtual uint32 GetDefaultNumInvocations(const UActorComponent* InComponent, int32 InLod) const override;
	virtual bool GetComponentElementCountsForExecutionDomain(
		FName InDomainName,
		const UActorComponent* InComponent,
		int32 InLodIndex,
		TArray<int32>& OutInvocationElementCounts) const override;
};
