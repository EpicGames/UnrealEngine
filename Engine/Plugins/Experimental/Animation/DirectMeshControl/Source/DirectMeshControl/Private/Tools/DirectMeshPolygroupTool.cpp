// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tools/DirectMeshPolygroupTool.h"

#include "BoneWeights.h"
#include "InteractiveToolManager.h"
#include "ToolBuilderUtil.h"
#include "ToolSetupUtil.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "PreviewMesh.h"

#include "MeshOpPreviewHelpers.h"
#include "ModelingOperators.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "DynamicMeshEditor.h"
#include "DynamicSubmesh3.h"
#include "MeshRegionBoundaryLoops.h"
#include "Util/ColorConstants.h"
#include "Polygroups/PolygroupUtil.h"
#include "Util/ColorConstants.h"
#include "Selections/GeometrySelectionUtil.h"
#include "Selection/StoredMeshSelectionUtil.h"
#include "Selection/GeometrySelectionVisualization.h"
#include "PropertySets/GeometrySelectionVisualizationProperties.h"
#include "ToolTargetManager.h"
#include "DynamicMesh/DynamicBoneAttribute.h"
#include "DynamicMesh/DynamicVertexSkinWeightsAttribute.h"

#include "CanvasTypes.h"
#include "Engine/Engine.h"  // for GEngine->GetSmallFont()

#include "Polygroups/PolygroupsGenerator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DirectMeshPolygroupTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UDirectMeshPolygroupTool"

/*
 * ToolBuilder
 */

USingleTargetWithSelectionTool* UDirectMeshPolygroupToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	return NewObject<UDirectMeshPolygroupTool>(SceneState.ToolManager);
}

bool UDirectMeshPolygroupToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return USingleTargetWithSelectionToolBuilder::CanBuildTool(SceneState) &&
		SceneState.TargetManager->CountSelectedAndTargetableWithPredicate(SceneState, GetTargetRequirements(),
			[](UActorComponent& Component) { return !ToolBuilderUtil::IsVolume(Component); }) >= 1;
}

class FDirectMeshPolygroupsGenerator : public FPolygroupsGenerator
{
public:

	struct FPolygroupsFromSkinWeightsParams
	{
		bool bTryUsingQuads = true;
		bool bOptimizeNeighbours = false;
		bool bRespectUVSeams = true;
		bool bRespectNormalSeams = false;
		double QuadAdjacencyWeight = 1.0;
		double QuadMetricClamp = 1.0;
		int MaxSearchRounds = 1;
		TArray<int32> BonesToRemove;
		TArray<int32> Parents;
	};

	FDirectMeshPolygroupsGenerator() {}
	FDirectMeshPolygroupsGenerator(FDynamicMesh3* MeshIn)
		: FPolygroupsGenerator(MeshIn) {}

	/**
	 * Find Polygroups based on skin weights
	 */
	bool FindPolygroupsFromSkinWeights(const FPolygroupsFromSkinWeightsParams& InParams)
	{
		using namespace UE::AnimationCore;
		
		FoundPolygroups.Reset();
		
		if (!Mesh->HasAttributes())
		{
			return false;
		}
		
		const FDynamicMeshVertexSkinWeightsAttribute* SkinWeights = Mesh->Attributes()->GetSkinWeightsAttribute("Default");
		if (!SkinWeights)
		{
			return false;
		}

		// store ids as quads
		TArray<TArray<int>> FoundQuads;
		if (InParams.bTryUsingQuads)
		{
			// do not copy on the mesh now
			TGuardValue<bool> CopyGuard(bCopyToMesh, false);
			if (FindSourceMeshPolygonPolygroups(
				InParams.bRespectUVSeams,
				InParams.bRespectNormalSeams,
				InParams.QuadAdjacencyWeight,
				InParams.QuadMetricClamp,
				InParams.MaxSearchRounds))
			{
				::Swap(FoundPolygroups, FoundQuads);
			}
		}

		static constexpr float Third = 1.f / 3.f;
	    static constexpr FBoneWeightsSettings BlendSettings;
		
		TArray<FBoneWeights> TriangleWeights;
		TriangleWeights.Reserve(Mesh->TriangleCount());

		// store per triangle bone weights
		const int32 NumTriangles = Mesh->TriangleCount();
		for (int32 Index = 0; Index < NumTriangles; Index++)
		{
			const FIndex3i& Tri = Mesh->GetTriangleRef(Index);

			FBoneWeights Weights0, Weights1, Weights2;
			SkinWeights->GetValue(Tri[0], Weights0);
			SkinWeights->GetValue(Tri[1], Weights1);
			SkinWeights->GetValue(Tri[2], Weights2);
			
			TriangleWeights.Add( FBoneWeights::Blend(Weights0, Weights1, Weights2, Third, Third, Third, BlendSettings) );
		}

		TMap<FBoneIndexType, TArray<int32>> TrianglesPerBone;

		TArray<FBoneIndexType> TriangleBone;
		TriangleBone.Init(0, NumTriangles);

		auto MoveToParentIndexIfRemoved = [&TriangleBone, &InParams](int32 TriIndex)
		{
			FBoneIndexType& BoneIndex = TriangleBone[TriIndex];
			if (BoneIndex != 0)
			{
				while (InParams.BonesToRemove.Contains(BoneIndex) && BoneIndex > 0)
				{
					if (ensure(InParams.Parents.IsValidIndex(BoneIndex)))
					{
						BoneIndex = FMath::Max(0, InParams.Parents[BoneIndex]);
					}
					else
					{
						BoneIndex = 0;
					}
				}
			}
		};
		
		if (FoundQuads.IsEmpty())
		{
			// store per bone triangles using all the triangles
			for (int32 TriIndex = 0; TriIndex < NumTriangles; TriIndex++)
			{
				const FBoneWeights& Weights = TriangleWeights[TriIndex];
				if (Weights.Num() > 0)
				{
					TriangleBone[TriIndex] = Weights[0].GetBoneIndex();
					MoveToParentIndexIfRemoved(TriIndex);
				}
			}
		}
		else
		{
			// store per bone quads
			for (const TArray<int>& QuadTris: FoundQuads)
			{
				if (QuadTris.Num() == 2)
				{
					const int32 TriIndex0 = QuadTris[0], TriIndex1 = QuadTris[1];
					const FBoneWeights& Weights0 = TriangleWeights[TriIndex0];
					const FBoneWeights& Weights1 = TriangleWeights[TriIndex1];

					const FBoneWeights& QuadWeights = FBoneWeights::Blend(Weights0, Weights1, 0.5f, BlendSettings);
					
					if (QuadWeights.Num() > 0)
					{
						const FBoneWeight BoneWeight = QuadWeights[0];
						TriangleBone[TriIndex0] = BoneWeight.GetBoneIndex();
						MoveToParentIndexIfRemoved(TriIndex0);
						TriangleBone[TriIndex1] = BoneWeight.GetBoneIndex();
						MoveToParentIndexIfRemoved(TriIndex1);
					}
				}
				else
				{
					for (const int32 TriIndex: QuadTris)
					{
						const FBoneWeights& Weights = TriangleWeights[TriIndex];
						if (Weights.Num() > 0)
						{
							TriangleBone[TriIndex] = Weights[0].GetBoneIndex();
							MoveToParentIndexIfRemoved(TriIndex);
						}
					}
				}
			}
		}

		for (int32 TriIndex = 0; TriIndex < NumTriangles; TriIndex++)
		{
			TArray<int32>& BoneTriangles = TrianglesPerBone.FindOrAdd(TriangleBone[TriIndex]);
			BoneTriangles.Add(TriIndex);
		}
		
		for (const auto&[BoneIndex, Triangles]: TrianglesPerBone)
		{
			FoundPolygroups.Add(Triangles);
		}

		if (bApplyPostProcessing)
		{
			// optimize to the parent ?
		}

		if (bCopyToMesh)
		{
			Mesh->EnableTriangleGroups(0);
			for (const auto&[BoneIndex, Triangles]: TrianglesPerBone)
			{
				for (const int32 TriID: Triangles)
				{
					Mesh->SetTriangleGroup(TriID, BoneIndex);
				}
			}
		}
		
		return !FoundPolygroups.IsEmpty();
	}
};

class FDirectMeshPolygroupOp : public  FDynamicMeshOperator
{
public:
	// parameters set by the tool
	int32 MinGroupSize = 2;
	int32 InitialGroupID = 0;
	FDirectMeshPolygroupsGenerator::FPolygroupsFromSkinWeightsParams Params;

	// input mesh
	TSharedPtr<FSharedConstDynamicMesh3, ESPMode::ThreadSafe> OriginalMesh;
	// input polygroups, if available
	TSharedPtr<FPolygroupSet, ESPMode::ThreadSafe> SourcePolyGroups;

	// result
	FDirectMeshPolygroupsGenerator Generator;

	virtual void CalculateResult(FProgressCancel* Progress) override
	{
		if ((Progress && Progress->Cancelled()) || OriginalMesh.IsValid() == false)
		{
			return;
		}

		OriginalMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
		{
			ResultMesh->Copy(ReadMesh, true, true, true, true);
		});
	
		if (Progress && Progress->Cancelled())
		{
			return;
		}

		Generator = FDirectMeshPolygroupsGenerator(ResultMesh.Get());
		Generator.MinGroupSize = MinGroupSize;
		Generator.InitialGroupID = InitialGroupID;
		
		Generator.FindPolygroupsFromSkinWeights(Params);

		Generator.FindPolygroupEdges();
	}

	void SetTransform(const FTransformSRT3d& Transform)
	{
		ResultTransform = Transform;
	}


};

namespace UE::Geometry
{

FDynamicMeshTriangleLabelAttribute* GetTriangleLabelAttribute(FDynamicMesh3& InMesh, const FName LayerName)
{
	checkSlow(InMesh.HasAttributes());

	FDynamicMeshAttributeSet* Attributes = InMesh.Attributes();
	FDynamicMeshTriangleLabelAttribute* Attr = Attributes->FindTriangleLabelAttribute(LayerName);
	if (!Attr)
	{
		Attr = new FDynamicMeshTriangleLabelAttribute(&InMesh);
		Attributes->AttachTriangleLabelAttribute(LayerName, Attr);
	}

	return Attr;
}
	
}

TUniquePtr<FDynamicMeshOperator> UDirectMeshPolygroupOperatorFactory::MakeNewOperator()
{
	// backpointer used to populate parameters.
	check(DirectMeshPolygroupTool);

	// Create the actual operator type based on the requested operation
	TUniquePtr<FDirectMeshPolygroupOp>  MeshOp = MakeUnique<FDirectMeshPolygroupOp>();

	// Operator runs on another thread - copy data over that it needs.
	DirectMeshPolygroupTool->UpdateOpParameters(*MeshOp);

	// give the operator
	return MeshOp;
}

/*
 * Tool
 */
UDirectMeshPolygroupTool::UDirectMeshPolygroupTool()
{}

bool UDirectMeshPolygroupTool::CanAccept() const
{
	return Super::CanAccept() && (PreviewCompute == nullptr || PreviewCompute->HaveValidResult());
}

void UDirectMeshPolygroupTool::Setup()
{
	UInteractiveTool::Setup();
	SetToolDisplayName(LOCTEXT("DirectMeshPolygroupToolName", "Generate PolyGroups"));
	
	FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);

	OriginalDynamicMesh = MakeShared<FDynamicMesh3, ESPMode::ThreadSafe>(UE::ToolTarget::GetDynamicMeshCopy(Target));

	// initialize triangle ROI if one exists
	SelectionTriangleROI = MakeShared<TSet<int32>, ESPMode::ThreadSafe>();
	TArray<int> TriangleROI;
	if (HasGeometrySelection())
	{
		const FGeometrySelection& InputSelection = GetGeometrySelection();

		EnumerateSelectionTriangles(InputSelection, *OriginalDynamicMesh, [&](int32 TriangleID)
		{
			SelectionTriangleROI->Add(TriangleID);
		});

		TriangleROI = SelectionTriangleROI->Array();

		OriginalSubmesh = MakeShared<FDynamicSubmesh3, ESPMode::ThreadSafe>(OriginalDynamicMesh.Get(), TriangleROI);
		bUsingSelection = true;
	}

	if (bUsingSelection)
	{
		ComputeOperatorSharedMesh = MakeShared<FSharedConstDynamicMesh3>(&OriginalSubmesh->GetSubmesh());
	}
	else
	{
		ComputeOperatorSharedMesh = MakeShared<FSharedConstDynamicMesh3>(OriginalDynamicMesh.Get());
	}

	Properties = NewObject<UDirectMeshPolygroupToolProperties>(this);
	Properties->RestoreProperties(this);
	AddToolPropertySource(Properties);
	FTransform MeshTransform = (FTransform)UE::ToolTarget::GetLocalToWorldTransform(Target);
	UE::ToolTarget::HideSourceObject(Target);

	{
		// create the operator factory
		UDirectMeshPolygroupOperatorFactory* DirectMeshPolygroupOperatorFactory = NewObject<UDirectMeshPolygroupOperatorFactory>(this);
		DirectMeshPolygroupOperatorFactory->DirectMeshPolygroupTool = this; // set the back pointer

		PreviewCompute = NewObject<UMeshOpPreviewWithBackgroundCompute>(DirectMeshPolygroupOperatorFactory);
		PreviewCompute->Setup(GetTargetWorld(), DirectMeshPolygroupOperatorFactory);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewCompute->PreviewMesh, Target);
		PreviewCompute->SetIsMeshTopologyConstant(true, EMeshRenderAttributeFlags::Positions | EMeshRenderAttributeFlags::VertexNormals);

		// Give the preview something to display
		PreviewCompute->PreviewMesh->SetTransform(MeshTransform);
		PreviewCompute->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
		PreviewCompute->PreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
		
		PreviewCompute->ConfigureMaterials(MaterialSet.Materials,
			ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
		);

		// show the preview mesh
		PreviewCompute->SetVisibility(true);

		// something to capture the polygons from the async task when it is done
		PreviewCompute->OnOpCompleted.AddLambda([this](const FDynamicMeshOperator* MeshOp) 
		{ 
			const FDirectMeshPolygroupOp*  DirectMeshPolygroupOp = static_cast<const FDirectMeshPolygroupOp*>(MeshOp);
			this->PolygonEdges = DirectMeshPolygroupOp->Generator.PolygroupEdges;
			UpdateVisualization();
		});

	}
	
	if (bUsingSelection)
	{
		// create the preview object for the unmodified area
		UnmodifiedAreaPreviewMesh = NewObject<UPreviewMesh>();
		UnmodifiedAreaPreviewMesh->CreateInWorld(GetTargetWorld(), MeshTransform);
		ToolSetupUtil::ApplyRenderingConfigurationToPreview(UnmodifiedAreaPreviewMesh, Target);
		UnmodifiedAreaPreviewMesh->SetMaterials(MaterialSet.Materials);
		UnmodifiedAreaPreviewMesh->EnableSecondaryTriangleBuffers( [this](const FDynamicMesh3* Mesh, int32 TriangleID) {
			return SelectionTriangleROI->Contains(TriangleID);
		});
		UnmodifiedAreaPreviewMesh->SetSecondaryBuffersVisibility(false);
		UnmodifiedAreaPreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
		UnmodifiedAreaPreviewMesh->UpdatePreview(OriginalDynamicMesh.Get());
	}


	PreviewGeometry = NewObject<UPreviewGeometry>(this);
	PreviewGeometry->CreateInWorld(GetTargetWorld(), MeshTransform);

	Properties->WatchProperty(Properties->bShowGroupColors, [this](bool) { UpdateVisualization(); });
	Properties->WatchProperty(Properties->MinGroupSize, [this](int32) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->QuadAdjacencyWeight, [this](float) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->QuadMetricClamp, [this](float) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->QuadSearchRounds, [this](int) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->bRespectUVSeams, [this](int) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->bRespectHardNormals, [this](int) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->bTryUsingQuads, [this](bool) { OnSettingsModified(); });
	Properties->WatchProperty(Properties->BonesToRemove, [this](TArray<FName>){ OnSettingsModified(); });

	if (bUsingSelection)
	{
		GeometrySelectionVizProperties = NewObject<UGeometrySelectionVisualizationProperties>(this);
		GeometrySelectionVizProperties->RestoreProperties(this);
		AddToolPropertySource(GeometrySelectionVizProperties);
		GeometrySelectionVizProperties->Initialize(this);
		GeometrySelectionVizProperties->bEnableShowTriangleROIBorder = true;
		GeometrySelectionVizProperties->SelectionElementType = static_cast<EGeometrySelectionElementType>(GeometrySelection.ElementType);
		GeometrySelectionVizProperties->SelectionTopologyType = static_cast<EGeometrySelectionTopologyType>(GeometrySelection.TopologyType);

		GeometrySelectionViz = NewObject<UPreviewGeometry>(this);
		GeometrySelectionViz->CreateInWorld(GetTargetWorld(), MeshTransform);
		
		InitializeGeometrySelectionVisualization(
			GeometrySelectionViz,
			GeometrySelectionVizProperties,
			*OriginalDynamicMesh,
			GeometrySelection,
			nullptr,
			nullptr,
			&TriangleROI);
	}

	// start the compute
	PreviewCompute->InvalidateResult();

	// updates the triangle color visualization
	UpdateVisualization();

	GetToolManager()->DisplayMessage(
		LOCTEXT("OnStartTool", "Cluster triangles of the Mesh into PolyGroups using various strategies"),
		EToolMessageLevel::UserNotification);
}

void UDirectMeshPolygroupTool::UpdateOpParameters(FDirectMeshPolygroupOp& DirectMeshPolygroupOp) const
{
	DirectMeshPolygroupOp.MinGroupSize = Properties->MinGroupSize;
	DirectMeshPolygroupOp.Params.QuadMetricClamp = Properties->QuadMetricClamp;
	DirectMeshPolygroupOp.Params.QuadAdjacencyWeight = Properties->QuadAdjacencyWeight;
	DirectMeshPolygroupOp.Params.MaxSearchRounds = FMath::Clamp(Properties->QuadSearchRounds, 1, 100);
	DirectMeshPolygroupOp.Params.bRespectUVSeams = Properties->bRespectUVSeams;
	DirectMeshPolygroupOp.Params.bRespectNormalSeams = Properties->bRespectHardNormals;
	DirectMeshPolygroupOp.Params.bTryUsingQuads = Properties->bTryUsingQuads;

	DirectMeshPolygroupOp.OriginalMesh = ComputeOperatorSharedMesh;

	DirectMeshPolygroupOp.Params.BonesToRemove.Reset();
	DirectMeshPolygroupOp.Params.Parents.Reset();
	
	TArray<FName> BoneNames;
	ComputeOperatorSharedMesh->AccessSharedObject([&BoneNames, &DirectMeshPolygroupOp](const FDynamicMesh3& ReadMesh)
	{
		if (const FDynamicMeshAttributeSet* Attributes = ReadMesh.Attributes())
		{
			if (Attributes->HasBones())
			{
				BoneNames = Attributes->GetBoneNames()->GetAttribValues();
				DirectMeshPolygroupOp.Params.Parents = Attributes->GetBoneParentIndices()->GetAttribValues(); 
			}
		}
	});

	ensure(!BoneNames.IsEmpty());
	
	for (int32 BoneIndex = 0; BoneIndex < BoneNames.Num(); BoneIndex++)
	{
		const FString BoneNameStr = BoneNames[BoneIndex].ToString();
		const bool bRemove = Properties->BonesToRemove.ContainsByPredicate([BoneNameStr](const FName& BoneToRemove)
		{
			return BoneNameStr.Contains(BoneToRemove.ToString());
		});
		if (bRemove)
		{
			DirectMeshPolygroupOp.Params.BonesToRemove.Add(BoneIndex);
		}
	}

	if (bUsingSelection)
	{
		DirectMeshPolygroupOp.InitialGroupID = OriginalDynamicMesh->MaxGroupID();
	}

	DirectMeshPolygroupOp.SetTransform(UE::ToolTarget::GetLocalToWorldTransform(Target));
}

void UDirectMeshPolygroupTool::OnShutdown(EToolShutdownType ShutdownType)
{
	Properties->SaveProperties(this);
	UE::ToolTarget::ShowSourceObject(Target);

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
		PreviewGeometry = nullptr;
	}

	if (UnmodifiedAreaPreviewMesh != nullptr)
	{
		UnmodifiedAreaPreviewMesh->Disconnect();
		UnmodifiedAreaPreviewMesh = nullptr;
	}

	if (PreviewCompute)
	{
		FDynamicMeshOpResult Result = PreviewCompute->Shutdown();
		if (ShutdownType == EToolShutdownType::Accept)
		{
			GetToolManager()->BeginUndoTransaction(LOCTEXT("DirectMeshPolygroupToolTransactionName", "Save DMC Layer"));

			FDynamicMesh3* DynamicMeshResult = Result.Mesh.Get();
			if (ensure(DynamicMeshResult != nullptr))
			{
				FDynamicMesh3 UseResultMesh = *OriginalDynamicMesh;
				if (UseResultMesh.HasAttributes() == false)
				{
					UseResultMesh.EnableAttributes();
				}
				
				// if we want to write to any layer other than default, we have to find or create it
				FDynamicMeshTriangleLabelAttribute* LabelAttribute = GetTriangleLabelAttribute(UseResultMesh, Properties->LayerName);
				if (ensure(LabelAttribute))
				{
					const FDynamicMeshAttributeSet* Attributes = UseResultMesh.Attributes();
					const TArray<FName> BoneNames =
						Attributes && Attributes->HasBones() ? Attributes->GetBoneNames()->GetAttribValues() : TArray<FName>();

					for (const int32 TriangleID : UseResultMesh.TriangleIndicesItr())
					{
						const int32 BoneIndex = DynamicMeshResult->GetTriangleGroup(TriangleID);
						LabelAttribute->SetValue(TriangleID, BoneNames.IsValidIndex(BoneIndex) ? BoneNames[BoneIndex] : NAME_None);
					}
					
					UE::ToolTarget::CommitDynamicMeshUpdate(Target, UseResultMesh, true);
				}
				else
				{
					// If we can't find or create the layer (which should not be possible) the tool is going to do nothing,
					// this is the safest option at this point
					UE_LOGF(LogGeometry, Warning, "Output group layer missing?");
				}
			}

			if (bUsingSelection)
			{
				// if the input was a group selection, that selection is no longer valid. But if we output
				// a triangle selection it should be converted to the group selection
				FGeometrySelection OutputSelection;
				for (int32 tid : *SelectionTriangleROI)
				{
					OutputSelection.Selection.Add(FGeoSelectionID::MeshTriangle(tid).Encoded());
				}
				SetToolOutputGeometrySelectionForTarget(this, Target, OutputSelection);
			}

			GetToolManager()->EndUndoTransaction();
		}
	}

	if (ComputeOperatorSharedMesh.IsValid())
	{
		ComputeOperatorSharedMesh->ReleaseSharedObject();
	}

	Super::OnShutdown(ShutdownType);
}

void UDirectMeshPolygroupTool::OnSettingsModified()
{
	PreviewCompute->InvalidateResult();
}


void UDirectMeshPolygroupTool::OnTick(float DeltaTime)
{
	Super::OnTick(DeltaTime);

	PreviewCompute->Tick(DeltaTime);
}

void UDirectMeshPolygroupTool::UpdateVisualization()
{
	if (!PreviewCompute)
	{
		return;
	}

	IMaterialProvider* MaterialTarget = Cast<IMaterialProvider>(Target);
	FComponentMaterialSet MaterialSet;
	
	if (Properties->bShowGroupColors)
	{
		UMaterialInstanceDynamic* VtxColorMaterial = ToolSetupUtil::GetVertexColorMaterial(GetToolManager());
		MaterialSet.Materials.Init(VtxColorMaterial, MaterialTarget->GetNumMaterials());
		
		PreviewCompute->PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
		{
			return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
		}, 
		UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	else
	{
		MaterialTarget->GetMaterialSet(MaterialSet);
		PreviewCompute->PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
	}
	
	PreviewCompute->ConfigureMaterials(MaterialSet.Materials, ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager()));

	FColor GroupLineColor = FColor::Red;
	float GroupLineThickness = 2.0f;

	ComputeOperatorSharedMesh->AccessSharedObject([&](const FDynamicMesh3& ReadMesh)
	{
		PreviewGeometry->CreateOrUpdateLineSet(TEXT("GroupBorders"), PolygonEdges.Num(), 
			[&](int32 k, TArray<FRenderableLine>& LinesOut) {
				FVector3d A, B;
				ReadMesh.GetEdgeV(PolygonEdges[k], A, B);
				LinesOut.Add(FRenderableLine(A, B, GroupLineColor, GroupLineThickness));
			}, 1);
	});
}



#undef LOCTEXT_NAMESPACE

