// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

/**
 * FDirectMeshControlCommands holds the registered UI commands for the Direct Mesh Control plugin.
 */
class DIRECTMESHCONTROL_API FDirectMeshControlCommands : public TCommands<FDirectMeshControlCommands>
{
public:
	FDirectMeshControlCommands();

	/** Global palette command. */
	TSharedPtr<FUICommandInfo> BeginDirectMeshControlTools;
	
	/** Command that activates the "DMC Polygroups" tool for generating bone-based polygroups. */
	TSharedPtr<FUICommandInfo> BeginDirectMeshPolygroupTool;

	/** Initialize commands. */
	virtual void RegisterCommands() override;
};