// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SkeletalMeshComponent.h"

#include "DirectMeshControlComponent.generated.h"

#define UE_API DIRECTMESHCONTROL_API

/**
 * UDirectMeshControlComponent is a transient skeletal mesh component that bypasses normal bone transform processing and applies mesh deformers directly.
 * Used by the DMC tools to render per-polygroup visualization sub-meshes.
 * Bone transforms are skipped so that the deformer drives the geometry entirely, enabling the per-polygroup overlay without interfering with the source skeleton.
 */
UCLASS(transient)
class UE_API UDirectMeshControlComponent : public USkeletalMeshComponent
{
	GENERATED_BODY()
 	
	/** UPrimitiveComponent override. */
	virtual void SendRenderDynamicData_Concurrent() override;
};

#undef UE_API