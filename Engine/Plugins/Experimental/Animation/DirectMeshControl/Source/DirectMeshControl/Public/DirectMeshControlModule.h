// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "SkeletalMeshModelingModeToolExtensions.h"

struct FExtensionToolDescription;
struct FExtensionToolQueryInfo;
class IControlRigEditor;
class FUICommandList;
class FExtender;
class FRigVMEditorBase;

/**
 * FDirectMeshControlModule is the main editor module for the Direct Mesh Control plugin.
 */
class FDirectMeshControlModule : public IModuleInterface, public ISkeletalMeshModelingModeToolExtension
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IModelingModeToolExtension implementation */
	virtual FText GetExtensionName() override;
	virtual FText GetToolSectionName() override;
	virtual bool GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut) override;
	virtual void GetExtensionTools(const FExtensionToolQueryInfo& InQueryInfo, TArray<FExtensionToolDescription>& OutTools) override;
};