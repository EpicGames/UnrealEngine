// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

/**
 * FDirectMeshControlStyle manages the Slate UI style set for the Direct Mesh Control plugin.
 * Provides static helpers to initialize, shut down, and access the shared FSlateStyleSet used by DMC editor tools and commands.
 */

class DIRECTMESHCONTROL_API FDirectMeshControlStyle
{
public:
	/** Creates and registers the DMC Slate style set. */
	static void Initialize();

	/** Unregisters and resets the DMC Slate style set. */
	static void Shutdown();

	/** Returns the shared DMC Slate style pointer. */
	static TSharedPtr<class ISlateStyle> Get();

	/** Returns the registered name of the DMC style set. */
	static FName GetStyleSetName();

private:
	/** The shared FSlateStyleSet instance owned by this class. */
	static TSharedPtr<class FSlateStyleSet> StyleSet;
};