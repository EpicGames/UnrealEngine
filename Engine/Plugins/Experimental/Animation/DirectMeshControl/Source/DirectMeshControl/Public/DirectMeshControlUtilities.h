// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/SkeletalMesh.h"
#include "UObject/SoftObjectPtr.h"

#include "DirectMeshControlUtilities.generated.h"

#define UE_API DIRECTMESHCONTROL_API

class UMaterialInstanceDynamic;
class UOptimusDeformer;
class UMaterial;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

/**
 * FGroupSubMeshes holds the per-polygroup sub skeletal meshes split from a source skeletal mesh by a triangle label layer.
 * Each entry in SubSkeletalMeshes element corresponds to one polygroup in the source mesh.
 * Sub-meshes are generated with a single root-bone binding and a SubToSource vertex attribute that maps sub-mesh vertices back to the
 * original mesh indices.
 */

USTRUCT()
struct UE_API FGroupSubMeshes
{
	GENERATED_BODY()
	
	/**
	 * Splits the input skeletal mesh into one skeletal mesh per polygroup
	 * @param InSkeletalMesh  Source skeletal mesh supplying the skeleton.
	 * @param Mesh            Dynamic mesh whose triangle label layer is read.
	 * @param LayerName       Name of the triangle label layer that stores polygroup IDs.
	 */
	void Rebuild(USkeletalMesh* InSkeletalMesh, const UE::Geometry::FDynamicMesh3* Mesh, const FName LayerName);
	
	/** Marks all sub-meshes as garbage and clears the mesh and index-map arrays. */
	void Reset();

	/** Returns the array of generated sub skeletal meshes. */
	const TArray<TObjectPtr<USkeletalMesh>>& GetSubSkeletalMeshes() const;
	
	/** Returns the map from polygroup ID to index into GetSubSkeletalMeshes(). */
	const TMap<int32, int32>& GetGroupIdToSubMesh() const;
	
private:
	
	/** Generated sub skeletal meshes, one entry per unique polygroup in the source layer. */
	UPROPERTY()
	TArray<TObjectPtr<USkeletalMesh>> SubSkeletalMeshes;
	
	/** Maps each polygroup ID to the corresponding index in SubSkeletalMeshes. */
	UPROPERTY()
	TMap<int32, int32> GroupIdToSubMesh;
};

/**
 * FSubSkeletalMeshData caches entry associating a (skeletal mesh, layer name) pair with a DDC hash and the resulting per-polygroup sub-meshes.
 */

USTRUCT()
struct FSubSkeletalMeshData
{
	GENERATED_BODY()
	
	/** Default constructor (produces an empty, invalid entry). */
	FSubSkeletalMeshData() = default;
	
	/**
	 * Constructs an entry for the given mesh and layer without yet computing sub-meshes or a DDC hash.
	 * @param InSkeletalMesh  Source skeletal mesh to track.
	 * @param InLayerName     Triangle label layer name to track.
	 */
	FSubSkeletalMeshData(USkeletalMesh* InSkeletalMesh, const FName InLayerName)
		: SkeletalMesh(InSkeletalMesh)
		, LayerName(InLayerName)
	{}

	/** Soft pointer to the source skeletal mesh. */
	UPROPERTY()
	TSoftObjectPtr<USkeletalMesh> SkeletalMesh = nullptr;
	
	/** Name of the triangle label layer this entry was built from. */
	UPROPERTY()
	FName LayerName = NAME_None;
	
	/** DDC hash of the source mesh data; used to detect stale cache entries. */
	UPROPERTY()
	uint32 Hash = 0;
	
	/** The generated per-polygroup sub-meshes for this (mesh, layer) pair. */
	UPROPERTY()
	FGroupSubMeshes GroupSubMeshes;

	/** Hash computation so that FSubSkeletalMeshData can be stored. */
	friend FORCEINLINE uint32 GetTypeHash(const FSubSkeletalMeshData& Data)
	{
		return HashCombine(GetTypeHash(Data.SkeletalMesh), GetTypeHash(Data.LayerName));
	}

	bool operator ==(const FSubSkeletalMeshData& Other) const
	{
		return SkeletalMesh == Other.SkeletalMesh && LayerName == Other.LayerName;
	}
};

/**
 * Utility functions for the Direct Mesh Control system.
 * Provides accessors for DMC materials, deformers, and cached sub-mesh data, as well as named constants used across the plugin.
 */

namespace UE::DMC
{
	/**
	 * Fetches or builds the FGroupSubMeshes for the given (mesh, layer) pair.
	 * @param InSkeletalMesh  Source skeletal mesh.
	 * @param Mesh            Dynamic mesh representation used for splitting.
	 * @param LayerName       Triangle label layer that encodes polygroup IDs.
	 * @return Reference to the cached (or freshly built) group sub-meshes.
	 */
	UE_API const FGroupSubMeshes& GetSubMeshes(USkeletalMesh* InSkeletalMesh, const Geometry::FDynamicMesh3* Mesh, const FName LayerName);
	
	/** Returns the vertex attribute name used to map sub-mesh vertices back to the source mesh. */
	UE_API const FName& GetSubToSourceAttrName();
	
	/** Returns the Optimus deformer variable name that controls the overlay color. */
	UE_API const FName& GetColorVarName();
	
	/** Lazy-loads and returns the DMC material or nullptr on failure. */
	UE_API UMaterial* GetMaterial();
	
	/** Lazy-loads and returns the DMC deformer or nullptr on failure. */
	UE_API UOptimusDeformer* GetDeformer();
}
	
#undef UE_API
