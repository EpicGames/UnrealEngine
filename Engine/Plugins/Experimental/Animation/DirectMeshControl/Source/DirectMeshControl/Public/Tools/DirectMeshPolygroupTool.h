// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleTargetWithSelectionTool.h"
#include "PreviewMesh.h"
#include "ModelingOperators.h"
#include "MeshOpPreviewHelpers.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "DynamicMesh/MeshSharingUtil.h"
#include "DirectMeshPolygroupTool.generated.h"

// predeclaration
class UDirectMeshPolygroupTool;
class FDirectMeshPolygroupOp;
class UPreviewGeometry;
namespace UE::Geometry
{
	class FDynamicMesh3;
	struct FDynamicSubmesh3;
}

/**
 * UDirectMeshPolygroupToolBuilder builds a UDirectMeshPolygroupTool instance for the given scene state.
 */
UCLASS()
class DIRECTMESHCONTROL_API UDirectMeshPolygroupToolBuilder : public USingleTargetWithSelectionToolBuilder
{
	GENERATED_BODY()
public:
	/**
	 * Creates and returns a new UDirectMeshPolygroupTool instance for the provided scene state.
	 * @param SceneState  Current tool-manager scene state.
	 * @return Newly created tool instance.
	 */
	virtual USingleTargetWithSelectionTool* CreateNewTool(const FToolBuilderState & SceneState) const override;

	/** Returns false, the polygroup tool does not require an active selection to launch. */
	virtual bool RequiresInputSelection() const override { return false; }

	/**
	 * Returns true when the scene state contains exactly one non-volume skeletal mesh component.
	 * @param SceneState  Current tool-manager scene state.
	 * @return Whether the tool can be built for the given selection.
	 */
	virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
};

/**
 * UDirectMeshPolygroupToolProperties represents editable properties controlling polygroup generation behavior and display.
 */

UCLASS()
class DIRECTMESHCONTROL_API UDirectMeshPolygroupToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** Bias for Quads that are adjacent to already-discovered Quads. Set to 0 to disable.  */
	UPROPERTY(EditAnywhere, Category = Quads, meta = (UIMin = 0, UIMax = 5, EditCondition = "bTryUsingQuads == true"))
	float QuadAdjacencyWeight = 1.0;

	/** Set to values below 1 to ignore less-likely triangle pairings */
	UPROPERTY(EditAnywhere, Category = Quads, meta = (AdvancedDisplay, UIMin = 0, UIMax = 1, EditCondition = "bTryUsingQuads == true"))
	float QuadMetricClamp = 1.0;

	/** Iteratively repeat quad-searching in uncertain areas, to try to slightly improve results */
	UPROPERTY(EditAnywhere, Category = Quads, meta = (AdvancedDisplay, UIMin = 1, UIMax = 5, EditCondition = "bTryUsingQuads == true"))
	int QuadSearchRounds = 1;

	/** If true, polygroup borders will not cross existing UV seams */
	UPROPERTY(EditAnywhere, Category = Quads, meta = (EditCondition = "bTryUsingQuads == true"))
	bool bRespectUVSeams = false;

	/** If true, polygroup borders will not cross existing hard normal seams */
	UPROPERTY(EditAnywhere, Category = Quads, meta = (EditCondition = "bTryUsingQuads == true"))
	bool bRespectHardNormals = false;

	/** Minimum number of triangles to include in a group. Any group containing fewer triangles will be merged with an adjacent group (if possible). */
	UPROPERTY(EditAnywhere, Category = Filtering, meta = (UIMin = "1", UIMax = "100", ClampMin = "1", ClampMax = "10000"))
	int32 MinGroupSize = 2;
	
	/** Display each group with a different auto-generated color */
	UPROPERTY(EditAnywhere, Category = Display)
	bool bShowGroupColors = true;

	/** If true, quads are used to defined skinned areas first */
	UPROPERTY(EditAnywhere, Category = Options)
	bool bTryUsingQuads = true;

	/** Bones to remove from the polygroup computation */
	UPROPERTY(EditAnywhere, Category = Options)
	TArray<FName> BonesToRemove;
	
	/** Name of the output layer */
	UPROPERTY(EditAnywhere, Category = Output)
	FName LayerName = "dmc-polygroup";
};

/**
 * Operator factory for the Direct Mesh Polygroup tool.
 */
UCLASS()
class DIRECTMESHCONTROL_API UDirectMeshPolygroupOperatorFactory : public UObject, public UE::Geometry::IDynamicMeshOperatorFactory
{
	GENERATED_BODY()

public:
	/**
	 * Creates a new FDirectMeshPolygroupOp operator populated with the current settings from DirectMeshPolygroupTool.
	 * @return Unique pointer to the configured operator ready for background execution.
	 */
	virtual TUniquePtr<UE::Geometry::FDynamicMeshOperator> MakeNewOperator() override;

	/** Back pointer to the owning tool, provides settings and mesh data for operator creation. */
	UPROPERTY()
	TObjectPtr<UDirectMeshPolygroupTool> DirectMeshPolygroupTool;
};

/**
 * UDirectMeshPolygroupTool generates bone-based polygroups on a skeletal mesh asynchronously and commits the result to a named triangle label layer on Accept.
 */
UCLASS()
class DIRECTMESHCONTROL_API UDirectMeshPolygroupTool : public USingleTargetWithSelectionTool
{
	GENERATED_BODY()

public:
	UDirectMeshPolygroupTool();

	/** Builds the preview mesh, registers properties, and starts the background compute operator. */
	virtual void Setup() override;

	/**
	 * Commits the computed polygroup layer to the skeletal mesh on Accept, or discards the result on Cancel.
	 * @param ShutdownType  Whether the tool is being accepted or canceled.
	 */
	virtual void OnShutdown(EToolShutdownType ShutdownType) override;

	/**
	 * Ticks the background compute operator and refreshes the visualization when a new result is available.
	 * @param DeltaTime  Time elapsed since the last tick.
	 */
	virtual void OnTick(float DeltaTime) override;

	/** Returns true, the polygroup tool supports cancellation. */
	virtual bool HasCancel() const override { return true; }

	/** Returns true, the polygroup tool supports acceptance. */
	virtual bool HasAccept() const override { return true; }

	/** Returns true when the background operator has produced a valid result that can be committed. */
	virtual bool CanAccept() const override;

	/**
	 * Copies the current properties values and bone data into DirectMeshPolygroupOp so that the operator uses up-to-date settings on the next compute pass.
	 * @param DirectMeshPolygroupOp  Operator instance to populate.
	 */
	void UpdateOpParameters(FDirectMeshPolygroupOp& DirectMeshPolygroupOp) const;

protected:

	/** Invalidates the preview compute, triggering a new background operator run. */
	void OnSettingsModified();

	/** Editable properties displayed in the Details panel. */
	UPROPERTY()
	TObjectPtr<UDirectMeshPolygroupToolProperties> Properties;

	/** Background compute manager that runs FDirectMeshPolygroupOp asynchronously. */
	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> PreviewCompute = nullptr;

	/** Geometry used to render polygroup border edges on the preview mesh. */
	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry = nullptr;

	/** Preview mesh that renders the non-selected (unmodified) portion of the source mesh when a selection ROI is active. */
	UPROPERTY()
	TObjectPtr<UPreviewMesh> UnmodifiedAreaPreviewMesh = nullptr;

	/** Thread-safe copy of the full input dynamic mesh, used as the base for all compute passes. */
	TSharedPtr<UE::Geometry::FDynamicMesh3, ESPMode::ThreadSafe> OriginalDynamicMesh;

	/** true when the tool was launched with an active triangle selection. */
	bool bUsingSelection = false;

	/** Triangle indices that form the selection ROI; populated only when bUsingSelection is true. */ 
	TSharedPtr<TSet<int32>> SelectionTriangleROI;

	/** Sub-mesh extracted from OriginalDynamicMesh for the selection ROI, populated only when bUsingSelection is true. */
	TSharedPtr<UE::Geometry::FDynamicSubmesh3, ESPMode::ThreadSafe> OriginalSubmesh;

	/** Shared mesh reference passed to the compute operator, points to the full mesh or the selection sub-mesh depending on bUsingSelection. */
	TSharedPtr<UE::Geometry::FSharedConstDynamicMesh3> ComputeOperatorSharedMesh;

	/** Edge indices (relative to ComputeOperatorSharedMesh) that form the detected polygroup boundaries, used for edge visualization. */
	TArray<int> PolygonEdges;

	/** Refreshes the polygroup edge overlay and per-group color visualization after a new compute result arrives. */
	void UpdateVisualization();
};
