// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DirectMeshControlRig : ModuleRules
{
	public DirectMeshControlRig(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DirectMeshControl",
				"AnimationCore",
				"ControlRig",
				"ComputeFramework",
				"Core",
				"CoreUObject",
				"DynamicMesh",
				"Engine",
				"GeometryCore",
				"MeshConversion",
				"OptimusCore",
				"RigVM"
			}
		);
	}
}
