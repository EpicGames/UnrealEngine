// Copyright Epic Games, Inc. All Rights Reserved.

#include "DirectMeshControlRigModule.h"

#include "ControlRigGizmoLibrary.h"
#include "DirectMeshControlComponent.h"
#include "DirectMeshControlProxyWrapper.h"
#include "Engine/SkeletalMesh.h"

#define LOCTEXT_NAMESPACE "FDirectMeshControlRigModule"

void FDirectMeshControlRigModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FDirectMeshControlRigModule::OnPostEngineInit);
}

void FDirectMeshControlRigModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

void FDirectMeshControlRigModule::OnPostEngineInit() const
{
	FShapeProxyComponentProviderRegistry& Registry = FShapeProxyComponentProviderRegistry::Get();
	Registry.RegisterProvider<UDirectMeshControlProxy, UDirectMeshControlComponent>(
		[](UObject* Proxy, UObject* Outer) -> UPrimitiveComponent*
		{
			UDirectMeshControlProxy* DMCProxy = Cast<UDirectMeshControlProxy>(Proxy);
			if (USkeletalMesh* SkeletalMesh = DMCProxy ? DMCProxy->GetTypedOuter<USkeletalMesh>() : nullptr)
			{	
				const FName UniqueName = MakeUniqueObjectName(Outer, UDirectMeshControlComponent::StaticClass(), TEXT("DirectMeshControlComponent_CtrlProxy"));
				UDirectMeshControlComponent* DMCComponent = NewObject<UDirectMeshControlComponent>(Outer, UniqueName);
				DMCComponent->SetSkeletalMesh(SkeletalMesh);
				return DMCComponent;
			}
			return nullptr; 
		},
		[](UObject* Proxy, UObject* Component)
		{
			UDirectMeshControlProxy* DMCProxy = Cast<UDirectMeshControlProxy>(Proxy);
			USkeletalMesh* SkeletalMesh = DMCProxy ? DMCProxy->GetTypedOuter<USkeletalMesh>() : nullptr;
			UDirectMeshControlComponent* DMCComponent = Cast<UDirectMeshControlComponent>(Component);
			if (SkeletalMesh && DMCComponent && DMCComponent->GetSkeletalMeshAsset() != SkeletalMesh)
			{
				DMCComponent->SetSkeletalMesh(SkeletalMesh);
				return true;
			}
			return false;
		});
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FDirectMeshControlRigModule, DirectMeshControlRig)