// Copyright Epic Games, Inc. All Rights Reserved.

#include "Units/RigUnit_DirectMeshControl.h"

#include "Units/Execution/RigUnit_DynamicHierarchy.h"
#include "ControlRig.h"
#include "ControlRigGizmoActor.h"
#include "DirectMeshControlProxyWrapper.h"
#include "DirectMeshControlUtilities.h"
#include "IControlRigObjectBinding.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "OptimusDeformer.h"
#include "OptimusDeformerInstance.h"
#include "Engine/SkeletalMesh.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Polygroups/GroupSetAdapter.h"
#include "Polygroups/PolygroupUtil.h"
#include "Util/ColorConstants.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RigUnit_DirectMeshControl)

namespace UE::DirectMeshControl::Private
{

bool GetDynamicMesh(const USkeletalMesh* InSkeletalMesh, FDynamicMesh3& OutDynaMesh)
{
	if (!InSkeletalMesh)
	{
		return false;
	}
	
	constexpr int32 LOD = 0;
	FMeshDescription* MeshDescription = InSkeletalMesh->HasMeshDescription(LOD) ? InSkeletalMesh->GetMeshDescription(LOD) : nullptr;
	if (!MeshDescription)
	{
		return false;
	}

	FMeshDescriptionToDynamicMesh Converter;
	Converter.bTransformVertexColorsLinearToSRGB = false;
	Converter.bVIDsFromNonManifoldMeshDescriptionAttr = true;
	constexpr bool bWantTangents = true;
	Converter.Convert(MeshDescription, OutDynaMesh, bWantTangents);

	return true;
}
	
}

FRigUnit_SetupShapeLibraryFromLayer_Execute()
{
	using namespace UE::DirectMeshControl::Private;
	using namespace UE::Geometry;
	
	FString ErrorMessage;

	GroupNames.Reset();
	
	constexpr bool bAllowOnlyConstructionEvent = true;
	if (!FRigUnit_DynamicHierarchyBase::IsValidToRunInContext(ExecuteContext, bAllowOnlyConstructionEvent, &ErrorMessage))
	{
		if(!ErrorMessage.IsEmpty())
		{
			UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("%s"), *ErrorMessage);
		}
		return;
	}

	if (!ExecuteContext.OnAddShapeLibraryDelegate.IsBound())
	{
		UE_CONTROLRIG_RIGUNIT_REPORT_ERROR(TEXT("OnAddShapeLibraryDelegate not bound"));
		return;
	}
	
	if (LayerName == NAME_None)
	{
		return;
	}
	
	USkeletalMeshComponent* SkeletalMeshComponent = ExecuteContext.UnitContext.RequestDataSource<USkeletalMeshComponent>(UControlRig::OwnerComponent);
	USkeletalMesh* SkeletalMesh = SkeletalMeshComponent ? SkeletalMeshComponent->GetSkeletalMeshAsset() : nullptr;

	FDynamicMesh3 DynaMesh;
	if (!GetDynamicMesh(SkeletalMesh, DynaMesh))
	{
		return;
	}

	FDynamicMeshTriangleLabelAttribute* DMCLabelAttr = FindTriangleLabelLayerByName(DynaMesh, LayerName);
	if (!DMCLabelAttr)
	{
		return;
	}
	
	FTriangleAttributeAdapter LabelAdapter(&DynaMesh, DMCLabelAttr);
	{
		TSet<FName> UniqueNames;
		for (const int32 TriID : DynaMesh.TriangleIndicesItr())
		{
			UniqueNames.Add(LabelAdapter.GetValue(TriID));
		}
		GroupNames = UniqueNames.Array();
	}

	if (!GroupNames.IsEmpty())
	{
		const FSoftObjectPath SkeletalMeshPath(SkeletalMesh);
		FString LibraryName(SkeletalMeshPath.GetSubPathUtf8String());
		LibraryName += LayerName.ToString();
	
		const FGroupSubMeshes& GroupSubMeshes = UE::DMC::GetSubMeshes(SkeletalMesh, &DynaMesh, LayerName);
		const TArray<TObjectPtr<USkeletalMesh>>& SubMeshes = GroupSubMeshes.GetSubSkeletalMeshes();
		
		UControlRigShapeLibrary* ShapeLibrary = [SkeletalMesh, LayerName, &SubMeshes]()
		{
			if (UObjectBase* Found = FindObjectWithOuter(SkeletalMesh, UControlRigShapeLibrary::StaticClass(), LayerName))
			{
				UControlRigShapeLibrary* ShapeLibrary = static_cast<UControlRigShapeLibrary*>(Found);
				auto ShouldReset = [ShapeLibrary, &SubMeshes]()
				{
					if (ShapeLibrary->Shapes.Num() != SubMeshes.Num())
					{
						return true;
					}
						
					return ShapeLibrary->Shapes.ContainsByPredicate([&SubMeshes](const FControlRigShapeDefinition& ShapeDefinition)
					{
						UObject* Proxy = ShapeDefinition.ShapeProxy.Get();
						USkeletalMesh* SkeletalMesh = Proxy ? Proxy->GetTypedOuter<USkeletalMesh>() : nullptr;
						if (!SkeletalMesh)
						{
							return true;
						}
						return !SubMeshes.Contains(SkeletalMesh);
					});
				};
				
				if (ShouldReset())
				{
					ShapeLibrary->Shapes.Reset();
				}
				return ShapeLibrary;
			}
			
			return NewObject<UControlRigShapeLibrary>(SkeletalMesh, LayerName, RF_Public | RF_Standalone);
		}();
		
		if (ShapeLibrary)
		{
			if (ShapeLibrary->Shapes.IsEmpty())
			{
				ShapeLibrary->DefaultShape.ShapeName = NAME_None;
				ShapeLibrary->Shapes.Reserve(GroupNames.Num());
				ShapeLibrary->DefaultMaterial = UE::DMC::GetMaterial();
				ShapeLibrary->XRayMaterial = UE::DMC::GetMaterial();
				ShapeLibrary->MaterialHoveredParameter = TEXT("Hovered");
				ShapeLibrary->MaterialHoveredColorParameter = TEXT("HoveredColor");
			
				for (const auto& [GroudID, SubMeshIndex]: GroupSubMeshes.GetGroupIdToSubMesh())
				{
					ensure(SubMeshes.IsValidIndex(SubMeshIndex));
				
					USkeletalMesh* SubSkeletalMesh = SubMeshes.IsValidIndex(SubMeshIndex) ? SubMeshes[SubMeshIndex].Get() : nullptr;
					if (ensure(SubSkeletalMesh))
					{
						FControlRigShapeDefinition& ShapeDefinition = ShapeLibrary->Shapes.Emplace_GetRef();
						ShapeDefinition.ShapeName = LabelAdapter.GetValueFromGroup(GroudID);
						ShapeDefinition.ShapeProxy = NewObject<UDirectMeshControlProxy>(SubSkeletalMesh, NAME_None, RF_Public | RF_Standalone);
						ShapeDefinition.PostSetupFunction = [](AControlRigShapeActor* InShape)
						{
							if (USkeletalMeshComponent* DMCComponent = InShape ? Cast<USkeletalMeshComponent>(InShape->ProxyComponent.Get()) : nullptr)
							{
								{
									// FIXME culling
									DMCComponent->bAllowCullDistanceVolume = false;
									DMCComponent->bNeverDistanceCull = true;
									DMCComponent->BoundsScale = 1000.0f;
								}
							
								// prepare source skm
								UControlRig* ControlRig = InShape->ControlRig.Get();
								const TSharedPtr<IControlRigObjectBinding>& ObjectBinding = ControlRig ? ControlRig->GetObjectBinding() : nullptr;
								USkeletalMeshComponent* BoundSkeletalMeshComponent =
										ObjectBinding ? Cast<USkeletalMeshComponent>(ObjectBinding->GetBoundObject()) : nullptr;
								if (BoundSkeletalMeshComponent)
								{
									BoundSkeletalMeshComponent->SetAlwaysUseMeshDeformer(true);
									BoundSkeletalMeshComponent->SetForcedLOD(1);
								}
								
								// add deformer
								DMCComponent->SetMeshDeformer(UE::DMC::GetDeformer());
								
								// set binding explicitly
								if (BoundSkeletalMeshComponent)
								{
									if (UOptimusDeformerInstanceSettings* Settings =
										Cast<UOptimusDeformerInstanceSettings>(DMCComponent->GetMeshDeformerInstanceSettings()))
									{
										TWeakObjectPtr<USkeletalMeshComponent> WeakDMCComponent(DMCComponent);
										TWeakObjectPtr<USkeletalMeshComponent> WeakBindingComponent(BoundSkeletalMeshComponent);
										
										Settings->ComponentResolver.BindWeakLambda(BoundSkeletalMeshComponent, [WeakDMCComponent, WeakBindingComponent](FName) -> UActorComponent*
										{
											// return the bound skeletal mesh component if valid 
											if (USkeletalMeshComponent* BindingComponent = WeakBindingComponent.Get())
											{
												return BindingComponent;
											}
												
											// look for the current one otherwise
											if (USkeletalMeshComponent* DMCComponent = WeakDMCComponent.Get())
											{
												AActor* ParentActor = DMCComponent->GetAttachParentActor();
												while (ParentActor)
												{
													TArray<USkeletalMeshComponent*> Components;
													ParentActor->GetComponents(USkeletalMeshComponent::StaticClass(), Components);
													if (!Components.IsEmpty())
													{
														return Components[0];
													}
													ParentActor = ParentActor->GetAttachParentActor();
												}
											}
											
											return nullptr;
										});

										if (UOptimusDeformerInstance* Instance = Cast<UOptimusDeformerInstance>(DMCComponent->GetMeshDeformerInstance()))
										{
											Instance->SetupFromDeformer(UE::DMC::GetDeformer());
										}
									}
								}
								
								DMCComponent->SetUpdateAnimationInEditor(true);
							}
						};
					}
				}
				
				Algo::SortBy(ShapeLibrary->Shapes, [](const FControlRigShapeDefinition& ShapeDefinition)
				{
					return ShapeDefinition.ShapeName;
				}, FNameLexicalLess());
			}

			constexpr bool bDoNotLogResults = false;
			ExecuteContext.OnAddShapeLibraryDelegate.Execute(&ExecuteContext, LibraryName, ShapeLibrary, bDoNotLogResults);
		}
	}
}
