// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "DirectMeshControlProxyWrapper.generated.h"

/**
 * UDirectMeshControlProxy is a transient proxy object used by FShapeProxyComponentProviderRegistry
 * to associate a UDirectMeshControlComponent with a Control Rig shape.
 */
UCLASS(transient)
class DIRECTMESHCONTROLRIG_API UDirectMeshControlProxy : public UObject
{
	GENERATED_BODY()
};
