// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * FDirectMeshControlRigModule is an editor module that integrates the Direct Mesh Control plugin with the Control Rig system.
 */
class FDirectMeshControlRigModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	/** Called after the engine has finished initializing. */
	void OnPostEngineInit() const;
};
