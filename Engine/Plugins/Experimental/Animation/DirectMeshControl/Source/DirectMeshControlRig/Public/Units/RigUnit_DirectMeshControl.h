// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Units/Execution/RigUnit_DynamicHierarchy.h"

#include "RigUnit_DirectMeshControl.generated.h"


/**
 * FRigUnit_SetupShapeLibraryFromLayer reads a polygroup triangle label layer from the source skeletal mesh, generates per-polygroup sub-meshes
 * and registers a UControlRigShapeLibrary out of it.
 */

USTRUCT(meta=(DisplayName="Set Shape Library from Layer", Keywords="Direct Mesh Control, Surface Selection", Varying))
struct FRigUnit_SetupShapeLibraryFromLayer : public FRigUnit_DynamicHierarchyBaseMutable
{
	GENERATED_BODY()

	FRigUnit_SetupShapeLibraryFromLayer()
	{
		LayerName = "dmc-polygroup";
	}

	RIGVM_METHOD()
	virtual void Execute() override;

	/* The layer to extract */
	UPROPERTY(meta = (Input))
	FName LayerName;
	
	/* The extracted groups */
	UPROPERTY(meta = (Output))
	TArray<FName> GroupNames;
};
